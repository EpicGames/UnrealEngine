{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LiveLinkPrestonMDR",
	"Description": "Live Link support for the Preston MDR-3 Motor Driver",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "LiveLinkPrestonMDR",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LiveLinkPrestonMDREditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}