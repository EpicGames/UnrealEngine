// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkPrestonMDRFactory.h"

#include "LiveLinkPrestonMDRSource.h"

#define LOCTEXT_NAMESPACE "LiveLinkPrestonMDRSourceFactory"

ULiveLinkPrestonMDRSourceFactory::FBuildCreationPanelDelegate ULiveLinkPrestonMDRSourceFactory::OnBuildCreationPanel;

FText ULiveLinkPrestonMDRSourceFactory::GetSourceDisplayName() const
{
	return LOCTEXT("SourceDisplayName", "PrestonMDR");
}

FText ULiveLinkPrestonMDRSourceFactory::GetSourceTooltip() const
{
	return LOCTEXT("SourceTooltip", "Creates a connection to a Preston MDR motor driver");
}

TSharedPtr<SWidget> ULiveLinkPrestonMDRSourceFactory::BuildCreationPanel(FOnLiveLinkSourceCreated InOnLiveLinkSourceCreated) const
{
	if (OnBuildCreationPanel.IsBound())
	{
		return OnBuildCreationPanel.Execute(this, InOnLiveLinkSourceCreated);
	}
	return TSharedPtr<SWidget>();
}

TSharedPtr<ILiveLinkSource> ULiveLinkPrestonMDRSourceFactory::CreateSource(const FString& ConnectionString) const
{
	FLiveLinkPrestonMDRConnectionSettings Settings;
	if (!ConnectionString.IsEmpty())
	{
		FLiveLinkPrestonMDRConnectionSettings::StaticStruct()->ImportText(*ConnectionString, &Settings, nullptr, EPropertyPortFlags::PPF_None, nullptr, FLiveLinkPrestonMDRConnectionSettings::StaticStruct()->GetName(), true);
	}
	return MakeShared<FLiveLinkPrestonMDRSource>(Settings);
}

FString ULiveLinkPrestonMDRSourceFactory::CreateConnectionString(const FLiveLinkPrestonMDRConnectionSettings& Settings)
{
	FString ConnectionString;
	FLiveLinkPrestonMDRConnectionSettings::StaticStruct()->ExportText(ConnectionString, &Settings, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr, true);
	return ConnectionString;
}

#undef LOCTEXT_NAMESPACE
