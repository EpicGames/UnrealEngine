// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkPrestonMDRRole.h"
#include "LiveLinkPrestonMDRTypes.h"

#define LOCTEXT_NAMESPACE "LiveLinkRole"

UScriptStruct* ULiveLinkPrestonMDRRole::GetStaticDataStruct() const
{
	return FLiveLinkPrestonMDRStaticData::StaticStruct();
}

UScriptStruct* ULiveLinkPrestonMDRRole::GetFrameDataStruct() const
{
	return FLiveLinkPrestonMDRFrameData::StaticStruct();
}

UScriptStruct* ULiveLinkPrestonMDRRole::GetBlueprintDataStruct() const
{
	return FLiveLinkPrestonMDRBlueprintData::StaticStruct();
}

bool ULiveLinkPrestonMDRRole::InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const
{
	bool bSuccess = false;

	FLiveLinkPrestonMDRBlueprintData* BlueprintData = OutBlueprintData.Cast<FLiveLinkPrestonMDRBlueprintData>();
	const FLiveLinkPrestonMDRStaticData* StaticData = InSourceData.StaticData.Cast<FLiveLinkPrestonMDRStaticData>();
	const FLiveLinkPrestonMDRFrameData* FrameData = InSourceData.FrameData.Cast<FLiveLinkPrestonMDRFrameData>();
	if (BlueprintData && StaticData && FrameData)
	{
		GetStaticDataStruct()->CopyScriptStruct(&BlueprintData->StaticData, StaticData);
		GetFrameDataStruct()->CopyScriptStruct(&BlueprintData->FrameData, FrameData);
		bSuccess = true;
	}

	return bSuccess;
}

FText ULiveLinkPrestonMDRRole::GetDisplayName() const
{
	return LOCTEXT("Preston MDR Role", "Preston MDR");
}

#undef LOCTEXT_NAMESPACE