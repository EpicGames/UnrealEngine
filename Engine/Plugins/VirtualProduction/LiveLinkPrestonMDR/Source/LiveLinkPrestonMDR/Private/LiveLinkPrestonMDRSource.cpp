// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkPrestonMDRSource.h"

#include "ILiveLinkClient.h"

#include "Interfaces/IPv4/IPv4Endpoint.h"

#include "LiveLinkPrestonMDRRole.h"
#include "LiveLinkPrestonMDRTypes.h"

#include "Sockets.h"

#define LOCTEXT_NAMESPACE "LiveLinkPrestonMDRSource"

DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkPrestonMDRSource, Log, All);

FLiveLinkPrestonMDRSource::FLiveLinkPrestonMDRSource(FLiveLinkPrestonMDRConnectionSettings InConnectionSettings)
	: SocketSubsystem(ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM))
	, ConnectionSettings(MoveTemp(InConnectionSettings))
	, LastTimeDataReceived(0.0)
	, MessageThreadShutdownTime(0.0)
	, SourceStatus(EPrestonSourceStatus::NotConnected)
{
	SourceMachineName = FText::Format(LOCTEXT("PrestonMDRMachineName", "{0}:{1}"), FText::FromString(ConnectionSettings.IPAddress), FText::AsNumber(ConnectionSettings.PortNumber, &FNumberFormattingOptions::DefaultNoGrouping()));
}

FLiveLinkPrestonMDRSource::~FLiveLinkPrestonMDRSource()
{
	bool bIsReadyToShutdown = false;
	while (!bIsReadyToShutdown)
	{
		bIsReadyToShutdown = ShutdownMessageThreadAndSocket();
	}
}

void FLiveLinkPrestonMDRSource::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid)
{
	Client = InClient;
	SubjectKey = FLiveLinkSubjectKey(InSourceGuid, ConnectionSettings.SubjectName);

	OpenConnection();
}

void FLiveLinkPrestonMDRSource::InitializeSettings(ULiveLinkSourceSettings* Settings)
{
	ILiveLinkSource::InitializeSettings(Settings);

	if (ULiveLinkPrestonMDRSourceSettings* MDRSettings = Cast<ULiveLinkPrestonMDRSourceSettings>(Settings))
	{
		SavedSourceSettings = MDRSettings;

		if (MessageThread)
		{
			MessageThread->SetIncomingDataMode_GameThread(SavedSourceSettings->IncomingDataMode);
		}
	}
	else
	{
		UE_LOGF(LogLiveLinkPrestonMDRSource, Warning, "Preston MDR Source coming from Preset is outdated. Consider recreating a Preston MDR Source. Configure it and resave as preset");
	}
}

void FLiveLinkPrestonMDRSource::OnSettingsChanged(ULiveLinkSourceSettings* Settings, const FPropertyChangedEvent& PropertyChangedEvent)
{
	ILiveLinkSource::OnSettingsChanged(Settings, PropertyChangedEvent);

	const FProperty* const MemberProperty = PropertyChangedEvent.MemberProperty;
	const FProperty* const Property = PropertyChangedEvent.Property;
	if (Property && MemberProperty && (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive))
	{
		if (SavedSourceSettings != nullptr)
		{
			const FName PropertyName = Property->GetFName();
			const FName MemberPropertyName = MemberProperty->GetFName();

			if (PropertyName == GET_MEMBER_NAME_CHECKED(ULiveLinkPrestonMDRSourceSettings, IncomingDataMode))
			{
				if (MessageThread)
				{
					MessageThread->SetIncomingDataMode_GameThread(SavedSourceSettings->IncomingDataMode);
				}
			}
			else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULiveLinkPrestonMDRSourceSettings, FocusEncoderRange))
			{
				if (PropertyName == GET_MEMBER_NAME_CHECKED(FEncoderRange, Min))
				{
					SavedSourceSettings->FocusEncoderRange.Min = FMath::Clamp(SavedSourceSettings->FocusEncoderRange.Min, (uint16)0, SavedSourceSettings->FocusEncoderRange.Max);
				}
				else if (PropertyName == GET_MEMBER_NAME_CHECKED(FEncoderRange, Max))
				{
					SavedSourceSettings->FocusEncoderRange.Max = FMath::Clamp(SavedSourceSettings->FocusEncoderRange.Max, SavedSourceSettings->FocusEncoderRange.Min, (uint16)0xFFFF);
				}
			}
			else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULiveLinkPrestonMDRSourceSettings, IrisEncoderRange))
			{
				if (PropertyName == GET_MEMBER_NAME_CHECKED(FEncoderRange, Min))
				{
					SavedSourceSettings->IrisEncoderRange.Min = FMath::Clamp(SavedSourceSettings->IrisEncoderRange.Min, (uint16)0, SavedSourceSettings->IrisEncoderRange.Max);
				}
				else if (PropertyName == GET_MEMBER_NAME_CHECKED(FEncoderRange, Max))
				{
					SavedSourceSettings->IrisEncoderRange.Max = FMath::Clamp(SavedSourceSettings->IrisEncoderRange.Max, SavedSourceSettings->IrisEncoderRange.Min, (uint16)0xFFFF);
				}
			}
			else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULiveLinkPrestonMDRSourceSettings, ZoomEncoderRange))
			{
				if (PropertyName == GET_MEMBER_NAME_CHECKED(FEncoderRange, Min))
				{
					SavedSourceSettings->ZoomEncoderRange.Min = FMath::Clamp(SavedSourceSettings->ZoomEncoderRange.Min, (uint16)0, SavedSourceSettings->ZoomEncoderRange.Max);
				}
				else if (PropertyName == GET_MEMBER_NAME_CHECKED(FEncoderRange, Max))
				{
					SavedSourceSettings->ZoomEncoderRange.Max = FMath::Clamp(SavedSourceSettings->ZoomEncoderRange.Max, SavedSourceSettings->ZoomEncoderRange.Min, (uint16)0xFFFF);
				}
			}
		}
		else
		{
			UE_LOGF(LogLiveLinkPrestonMDRSource, Warning, "Preston MDR Source coming from Preset is outdated. Consider recreating a Preston MDR Source. Configure it and resave as preset");
		}
	}
}

bool FLiveLinkPrestonMDRSource::IsSourceStillValid() const
{
	return (SourceStatus == EPrestonSourceStatus::ConnectedActive || SourceStatus == EPrestonSourceStatus::ConnectedIdle);
}

bool FLiveLinkPrestonMDRSource::RequestSourceShutdown()
{
	{
		FScopeLock Lock(&PrestonSourceCriticalSection);
		SourceStatus = EPrestonSourceStatus::ShuttingDown;
	}

	return ShutdownMessageThreadAndSocket();
}

bool FLiveLinkPrestonMDRSource::ShutdownMessageThreadAndSocket()
{
	if (MessageThread.IsValid())
	{
		// Instruct the message thread to stop its message loop
		if (MessageThread->IsThreadRunning())
		{
			MessageThreadShutdownTime = FPlatformTime::Seconds();
			MessageThread->Stop();
		}

		if ((FPlatformTime::Seconds() - MessageThreadShutdownTime) > MessageThreadShutdownTimeout)
		{
			MessageThread->ForceKill();
		}

		// If the message thread has stopped executing its message loop, then it is safe to reset it and destroy the socket
		if (!MessageThread->IsFinished())
		{
			return false;
		}

		MessageThread.Reset();
	}

	if (Socket)
	{
		if (!Socket->Close())
		{
			UE_LOGF(LogLiveLinkPrestonMDRSource, Warning, "Preston MDR Source socket failed to close.");
		}

		SocketSubsystem->DestroySocket(Socket);
		Socket = nullptr;
	}

	return true;
}

FText FLiveLinkPrestonMDRSource::GetSourceType() const
{
	return LOCTEXT("PrestonMDRSourceType", "PrestonMDR");
}

void FLiveLinkPrestonMDRSource::Update()
{
	if (SourceStatus == EPrestonSourceStatus::ConnectedActive && (FPlatformTime::Seconds() - LastTimeDataReceived > DataReceivedTimeout))
	{
		{
			FScopeLock Lock(&PrestonSourceCriticalSection);
			SourceStatus = EPrestonSourceStatus::ConnectedIdle;
		}
	}
	else if (SourceStatus == EPrestonSourceStatus::ConnectionLost)
	{
		// If the connection was lost, shut down the message thread and the socket and attempt to reconnect
		if (ShutdownMessageThreadAndSocket())
		{
			SourceStatus = EPrestonSourceStatus::NotConnected;
			OpenConnection();
		}
	}
}

FText FLiveLinkPrestonMDRSource::GetSourceStatus() const
{
	switch (SourceStatus)
	{
	case EPrestonSourceStatus::NotConnected:
	{
		return LOCTEXT("NotConnectedStatus", "Not Connected");
	}
	case EPrestonSourceStatus::WaitingToConnect:
	{
		return LOCTEXT("WaitingToConnectStatus", "Waiting To Connect...");
	}
	case EPrestonSourceStatus::ConnectedActive:
	{
		return LOCTEXT("ConnectedActiveStatus", "Active");
	}
	case EPrestonSourceStatus::ConnectedIdle:
	{
		return LOCTEXT("ConnectedIdleStatus", "Idle");
	}
	case EPrestonSourceStatus::ConnectionFailed:
	{
		return LOCTEXT("ConnectionFailedStatus", "Failed To Connect");
	}
	case EPrestonSourceStatus::ConnectionLost:
	{
		return LOCTEXT("ConnectionLostStatus", "Connection Lost");
	}
	case EPrestonSourceStatus::ShuttingDown:
	{
		return LOCTEXT("ShuttingDownStatus", "Shutting Down...");
	}
	default:
	{
		return LOCTEXT("UnknownStatus", "Unknown");
	}
	}
}

void FLiveLinkPrestonMDRSource::OpenConnection()
{
	// Create an IPv4 TCP Socket
	Socket = SocketSubsystem->CreateSocket(NAME_Stream, TEXT("Preston MDR Socket"), FNetworkProtocolTypes::IPv4);
	Socket->SetNonBlocking(true);
	Socket->SetNoDelay(true);
	Socket->SetRecvErr(true);

	FIPv4Address IPAddr;
	if (!FIPv4Address::Parse(ConnectionSettings.IPAddress, IPAddr))
	{
		SourceStatus = EPrestonSourceStatus::ConnectionFailed;
		UE_LOGF(LogLiveLinkPrestonMDRSource, Error, "Ill-formed IP Address");
		return;
	}

	FIPv4Endpoint Endpoint = FIPv4Endpoint(IPAddr, ConnectionSettings.PortNumber);
	TSharedRef<FInternetAddr> Addr = Endpoint.ToInternetAddr();

	UE_LOGF(LogLiveLinkPrestonMDRSource, VeryVerbose, "Connecting to the MDR server at %ls", *Endpoint.ToString());

	if (!Socket->Connect(*Addr))
	{
		SourceStatus = EPrestonSourceStatus::ConnectionFailed;
		UE_LOGF(LogLiveLinkPrestonMDRSource, Error, "Could not connect to the MDR server at %ls", *Endpoint.ToString());
		return;
	}

	if (Socket->GetConnectionState() == ESocketConnectionState::SCS_ConnectionError)
	{
		SourceStatus = EPrestonSourceStatus::ConnectionFailed;
		UE_LOGF(LogLiveLinkPrestonMDRSource, Error, "Could not connect to the MDR server at %ls", *Endpoint.ToString());
		return;
	}

	// The socket is non-blocking, so we will not know if the connection was successful until sometime later
	SourceStatus = EPrestonSourceStatus::WaitingToConnect;

	MessageThread = MakeUnique<FPrestonMDRMessageThread>(Socket);

	MessageThread->OnFrameDataReady().BindRaw(this, &FLiveLinkPrestonMDRSource::OnFrameDataReady);
	MessageThread->OnStatusChanged().BindRaw(this, &FLiveLinkPrestonMDRSource::OnStatusChanged);
	MessageThread->OnConnectionLost().BindRaw(this, &FLiveLinkPrestonMDRSource::OnConnectionLost);
	MessageThread->OnConnectionFailed().BindRaw(this, &FLiveLinkPrestonMDRSource::OnConnectionFailed);

	MessageThread->Start();
}

void FLiveLinkPrestonMDRSource::OnConnectionLost()
{
	{
		FScopeLock Lock(&PrestonSourceCriticalSection);
		SourceStatus = EPrestonSourceStatus::ConnectionLost;
	}

	UE_LOGF(LogLiveLinkPrestonMDRSource, Error, "Connection to the MDR device was lost");	
}

void FLiveLinkPrestonMDRSource::OnConnectionFailed()
{
	{
		FScopeLock Lock(&PrestonSourceCriticalSection);
		SourceStatus = EPrestonSourceStatus::ConnectionFailed;
	}

	UE_LOGF(LogLiveLinkPrestonMDRSource, Error, "Could not connect to the MDR server at %ls:%d", *ConnectionSettings.IPAddress, ConnectionSettings.PortNumber);
}

void FLiveLinkPrestonMDRSource::OnStatusChanged(FMDR3Status InStatus)
{
	LatestMDRStatus = InStatus;
	UpdateStaticData();
}

void FLiveLinkPrestonMDRSource::UpdateStaticData()
{
	FLiveLinkStaticDataStruct PrestonMDRStaticDataStruct(FLiveLinkPrestonMDRStaticData::StaticStruct());
	FLiveLinkPrestonMDRStaticData* PrestonMDRStaticData = PrestonMDRStaticDataStruct.Cast<FLiveLinkPrestonMDRStaticData>();

	PrestonMDRStaticData->bIsFocalLengthSupported = LatestMDRStatus.bIsZoomMotorSet;
	PrestonMDRStaticData->bIsApertureSupported = LatestMDRStatus.bIsIrisMotorSet;
	PrestonMDRStaticData->bIsFocusDistanceSupported = LatestMDRStatus.bIsFocusMotorSet;

	PrestonMDRStaticData->bIsFieldOfViewSupported = false;
	PrestonMDRStaticData->bIsAspectRatioSupported = false;
	PrestonMDRStaticData->bIsProjectionModeSupported = false;

	Client->PushSubjectStaticData_AnyThread(SubjectKey, ULiveLinkPrestonMDRRole::StaticClass(), MoveTemp(PrestonMDRStaticDataStruct));
}

void FLiveLinkPrestonMDRSource::OnFrameDataReady(FLensDataPacket InData)
{
	{
		FScopeLock Lock(&PrestonSourceCriticalSection);
		SourceStatus = EPrestonSourceStatus::ConnectedActive;
	}

	FLiveLinkFrameDataStruct LensFrameDataStruct(FLiveLinkPrestonMDRFrameData::StaticStruct());
	FLiveLinkPrestonMDRFrameData* LensFrameData = LensFrameDataStruct.Cast<FLiveLinkPrestonMDRFrameData>();

	LastTimeDataReceived = FPlatformTime::Seconds();
	LensFrameData->WorldTime = LastTimeDataReceived.load();
	LensFrameData->MetaData.SceneTime = InData.FrameTime;

	if (SavedSourceSettings->IncomingDataMode == EFIZDataMode::CalibratedData)
	{
		LensFrameData->RawFocusEncoderValue = 0;
		LensFrameData->RawIrisEncoderValue = 0;
		LensFrameData->RawZoomEncoderValue = 0;

		LensFrameData->FocusDistance = InData.Focus;
		LensFrameData->Aperture = InData.Iris;
		LensFrameData->FocalLength = InData.Zoom;
	}
	else
	{
		LensFrameData->RawFocusEncoderValue = InData.Focus;
		LensFrameData->RawIrisEncoderValue = InData.Iris;
		LensFrameData->RawZoomEncoderValue = InData.Zoom;

		const uint16 FocusDelta = SavedSourceSettings->FocusEncoderRange.Max - SavedSourceSettings->FocusEncoderRange.Min;
		LensFrameData->FocusDistance = (FocusDelta != 0) ? FMath::Clamp((InData.Focus - SavedSourceSettings->FocusEncoderRange.Min) / (float)FocusDelta, 0.0f, 1.0f) : 0.0f;

		const uint16 IrisDelta = SavedSourceSettings->IrisEncoderRange.Max - SavedSourceSettings->IrisEncoderRange.Min;
		LensFrameData->Aperture = (IrisDelta != 0) ? FMath::Clamp((InData.Iris - SavedSourceSettings->IrisEncoderRange.Min) / (float)IrisDelta, 0.0f, 1.0f) : 0.0f;

		const uint16 ZoomDelta = SavedSourceSettings->ZoomEncoderRange.Max - SavedSourceSettings->ZoomEncoderRange.Min;
		LensFrameData->FocalLength = (ZoomDelta != 0) ? FMath::Clamp((InData.Zoom - SavedSourceSettings->ZoomEncoderRange.Min) / (float)ZoomDelta, 0.0f, 1.0f) : 0.0f;
	}

	Client->PushSubjectFrameData_AnyThread(SubjectKey, MoveTemp(LensFrameDataStruct));
}

#undef LOCTEXT_NAMESPACE
