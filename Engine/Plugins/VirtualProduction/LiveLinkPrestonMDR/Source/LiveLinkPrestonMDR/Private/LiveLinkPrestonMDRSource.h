// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkSource.h"

#include "LiveLinkPrestonMDRConnectionSettings.h"

#include "PrestonMDRMessageThread.h"


class ULiveLinkSourceSettings;

class ISocketSubsystem;

enum class EPrestonSourceStatus : uint8
{
	NotConnected,
	WaitingToConnect,
	ConnectedActive,
	ConnectedIdle,
	ConnectionFailed,
	ConnectionLost,
	ShuttingDown
};

class LIVELINKPRESTONMDR_API FLiveLinkPrestonMDRSource : public ILiveLinkSource
{
public:

	FLiveLinkPrestonMDRSource(FLiveLinkPrestonMDRConnectionSettings ConnectionSettings);
	~FLiveLinkPrestonMDRSource();

	// Begin ILiveLinkSource Implementation
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;
	virtual void Update() override;
	virtual bool IsSourceStillValid() const override;

	virtual bool RequestSourceShutdown() override;

	virtual FText GetSourceType() const override;
	virtual FText GetSourceMachineName() const override { return SourceMachineName; }
	virtual FText GetSourceStatus() const override;

	virtual TSubclassOf<ULiveLinkSourceSettings> GetSettingsClass() const { return ULiveLinkPrestonMDRSourceSettings::StaticClass(); }

	virtual void InitializeSettings(ULiveLinkSourceSettings* Settings) override;
	virtual void OnSettingsChanged(ULiveLinkSourceSettings* Settings, const FPropertyChangedEvent& PropertyChangedEvent) override;
	// End ILiveLinkSourceImplementation

private:
	void OpenConnection();
	void OnFrameDataReady(FLensDataPacket InData);
	void OnStatusChanged(FMDR3Status InStatus);
	void OnConnectionLost();
	void OnConnectionFailed();
	void UpdateStaticData();
	bool ShutdownMessageThreadAndSocket();

	ILiveLinkClient* Client = nullptr;
	FSocket* Socket = nullptr;
	ISocketSubsystem* SocketSubsystem = nullptr;

	FLiveLinkPrestonMDRConnectionSettings ConnectionSettings;
	FLiveLinkSubjectKey SubjectKey;
	FText SourceMachineName;

	TObjectPtr<ULiveLinkPrestonMDRSourceSettings> SavedSourceSettings = nullptr;

	TUniquePtr<FPrestonMDRMessageThread> MessageThread;

	FCriticalSection PrestonSourceCriticalSection;

	std::atomic<double> LastTimeDataReceived;
	double MessageThreadShutdownTime;

	EPrestonSourceStatus SourceStatus = EPrestonSourceStatus::NotConnected;

	FMDR3Status LatestMDRStatus;

private:
	const float DataReceivedTimeout = 1.0f;
	const float MessageThreadShutdownTimeout = 2.0f;
};
