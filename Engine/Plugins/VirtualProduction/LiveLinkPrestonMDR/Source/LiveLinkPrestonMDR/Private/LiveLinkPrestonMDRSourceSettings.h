// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkSourceSettings.h"


#include "LiveLinkPrestonMDRSourceSettings.generated.h"

USTRUCT()
struct FEncoderRange
{
	GENERATED_BODY()

	/** Minimum raw encoder value */
	UPROPERTY(EditAnywhere, Category = "Encoder Data")
	uint16 Min = 0x0000;

	/** Maximum raw encoder value */
	UPROPERTY(EditAnywhere, Category = "Encoder Data")
	uint16 Max = 0xFFFF;
};

UENUM()
enum class EFIZDataMode : uint8
{
	EncoderData = 0,
	CalibratedData = 1
};

UCLASS()
class LIVELINKPRESTONMDR_API ULiveLinkPrestonMDRSourceSettings : public ULiveLinkSourceSettings
{
public:
	GENERATED_BODY()

	/** The mode in which the Preston MDR is configured to send FIZ data (pre-calibrated or raw encoder positions) */
	UPROPERTY(EditAnywhere, Category = "Source")
	EFIZDataMode IncomingDataMode = EFIZDataMode::EncoderData;

	/** Raw focus encoder range */
	UPROPERTY(EditAnywhere, Category = "Source")
	FEncoderRange FocusEncoderRange;

	/** Raw iris encoder range */
	UPROPERTY(EditAnywhere, Category = "Source")
	FEncoderRange IrisEncoderRange;

	/** Raw zoom encoder range */
	UPROPERTY(EditAnywhere, Category = "Source")
	FEncoderRange ZoomEncoderRange;
};
