// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Roles/LiveLinkCameraTypes.h"

#include "LiveLinkPrestonMDRTypes.generated.h"

/**
 * Struct for static Preston MDR data
 */
USTRUCT(BlueprintType)
struct LIVELINKPRESTONMDR_API FLiveLinkPrestonMDRStaticData : public FLiveLinkCameraStaticData
{
	GENERATED_BODY()
};

/**
 * Struct for dynamic (per-frame) Preston MDR data
 */
USTRUCT(BlueprintType)
struct LIVELINKPRESTONMDR_API FLiveLinkPrestonMDRFrameData : public FLiveLinkCameraFrameData
{
	GENERATED_BODY()

	/** Raw encoder value for focus motor */
	UPROPERTY(VisibleAnywhere, Category = "Raw Encoder Values")
	uint16 RawFocusEncoderValue = 0;

	/** Raw encoder value for iris motor */
	UPROPERTY(VisibleAnywhere, Category = "Raw Encoder Values")
	uint16 RawIrisEncoderValue = 0;

	/** Raw encoder value for zoom motor */
	UPROPERTY(VisibleAnywhere, Category = "Raw Encoder Values")
	uint16 RawZoomEncoderValue = 0;
};

/**
 * Facility structure to handle Preston MDR data in blueprint
 */
USTRUCT(BlueprintType)
struct LIVELINKPRESTONMDR_API FLiveLinkPrestonMDRBlueprintData : public FLiveLinkBaseBlueprintData
{
	GENERATED_BODY()

	/** Static data that should not change every frame */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FLiveLinkPrestonMDRStaticData StaticData;

	/** Dynamic data that can change every frame  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FLiveLinkPrestonMDRFrameData FrameData;
};
