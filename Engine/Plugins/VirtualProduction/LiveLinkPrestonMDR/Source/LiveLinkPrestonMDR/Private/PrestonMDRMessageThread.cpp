// Copyright Epic Games, Inc. All Rights Reserved.

#include "PrestonMDRMessageThread.h"

#include "Sockets.h"

#include "HAL/RunnableThread.h"
#include "Misc/ScopeLock.h"

DEFINE_LOG_CATEGORY_STATIC(LogPrestonMDRMessageThread, Log, All);

const uint8 FMDRProtocol::STX = 0x02;
const uint8 FMDRProtocol::ETX = 0x03;
const uint8 FMDRProtocol::ACK = 0x06;

const uint8 FMDRProtocol::StatusMessageLength = 8;
const uint8 FMDRProtocol::DataMessageLength = 10;
const uint8 FMDRProtocol::TimeMessageLength = 10;
const uint8 FMDRProtocol::TimecodeMessageLength = 8;

FPrestonMDRMessageThread::FPrestonMDRMessageThread(FSocket* InSocket) :
	Socket(InSocket)
{
}

FPrestonMDRMessageThread::~FPrestonMDRMessageThread()
{
	if (Thread != nullptr)
	{
		Thread->Kill(true);
	}
}

void FPrestonMDRMessageThread::SetSocket(FSocket* InSocket)
{
	Socket = InSocket;
	bSoftResetTriggered = false;
	bHardResetTriggered = false;
	LastTimeDataReceived = FPlatformTime::Seconds();
}

void FPrestonMDRMessageThread::SetIncomingDataMode_GameThread(EFIZDataMode InDataMode)
{
	DataDescriptionSettings.LensDataMode = InDataMode;
	UpdateDataDescriptionMessage_AnyThread();
}

void FPrestonMDRMessageThread::Start()
{
	bIsFinished = false;

	Thread.Reset(FRunnableThread::Create(this, TEXT("Preston MDR Message Thread"), ThreadStackSize, TPri_AboveNormal));
}

bool FPrestonMDRMessageThread::Init()
{
	bIsThreadRunning = true;
	return true;
}

void FPrestonMDRMessageThread::Stop()
{
	bIsThreadRunning = false;
}

uint32 FPrestonMDRMessageThread::Run()
{
	// Initialize the messages that will be sent to the MDR
	BuildStaticMessages();

	// Fill the command queue with a message sequence to start the communication flow with the MDR
	InitializeCommandQueue();

	// Reserve space in the FIZ data array to hold the current values of the Focus, Iris, Zoom, and AUX motors
	constexpr uint8 MaxNumFIZDataPoints = 4;
	ParsedFIZData.SetNumZeroed(MaxNumFIZDataPoints);

	// Reserve space for the maximum amount of data that we can receive on the socket at once
	TArray<uint8> ReceiveBuffer;
	ReceiveBuffer.SetNumZeroed(MaximumMDRMessageLength);

	// Initialize the timer used to tell if the socket connection has gone idle
	LastTimeDataReceived = FPlatformTime::Seconds();

	// Wait for the Socket to be fully connected (in a readable and writable state)
	while (Socket->GetConnectionState() != ESocketConnectionState::SCS_Connected)
	{
		if ((FPlatformTime::Seconds() - LastTimeDataReceived) > ConnectionTimeout)
		{
			ConnectionFailedDelegate.ExecuteIfBound();
			bIsFinished = true;
			return 0;
		}

		// Yield execution of this thread for the specified interval
		FPlatformProcess::Sleep(ConnectionWaitInterval);
	}

	LastTimeDataReceived = FPlatformTime::Seconds();

	int32 TotalBytesRead = 0;
	uint32 NumBytesOnSocket = 0;

	uint32 NumCommandsSent = 0;
	uint32 NumRepliedReceived = 0;

	while ((bIsThreadRunning || !bReadyToSendCommandToServer) && !bForceKillThread)
	{
		if (bReadyToSendCommandToServer)
		{
			bReadyToSendCommandToServer = false;
			SendNextCommandToServer();
			NumCommandsSent++;
		}

		int32 NumBytesRead = 0;
		if (Socket->Recv(ReceiveBuffer.GetData() + TotalBytesRead, MaximumMDRMessageLength - TotalBytesRead, NumBytesRead))
		{
			TotalBytesRead += NumBytesRead;

			// Attempt to break up packets from the server until there are no full packets remaining in the buffer
			while (true)
			{
				// Minimum packet length, assuming a zero-length payload
				uint8 PacketLength = FMDRProtocol::HeaderSize + FMDRProtocol::FooterSize;

				// Ensure that we received at least the minimum amount of data to constitute a valid packet before trying to parse it
				if (TotalBytesRead < PacketLength)
				{
					break;
				}

				// Read the packet header to determine how large the payload is expected to be
				uint8 NumPayloadBytes = 2 * AsciiToHex(&ReceiveBuffer[FMDRProtocol::PayloadLengthByte]); // There are two ASCII-encoded bytes per payload byte
				PacketLength += NumPayloadBytes;

				// Ensure that we received at least the minimum amount of data to constitute a valid packet before trying to parse it
				if (TotalBytesRead < PacketLength)
				{
					break;
				}

				// Update the timer
				LastTimeDataReceived = FPlatformTime::Seconds();

				ParseMessageFromServer(ReceiveBuffer.GetData(), PacketLength);
				NumRepliedReceived++;

				// After receiving a new timecode and new FIZ data packet, send a new frame of lens data 
				if (bIsFrameTimeReady && bIsFrameDataReady)
				{
					bIsFrameTimeReady = false;
					bIsFrameDataReady = false;
					FrameDataReadyDelegate.ExecuteIfBound(LensData);
				}

				// Reply the the MDR server, acknowledging receipt of the previous packet
				SendMessageToServer(&FMDRProtocol::ACK, 1);
				bReadyToSendCommandToServer = true;

				ReceiveBuffer.RemoveAt(0, PacketLength, EAllowShrinking::No);
				ReceiveBuffer.AddZeroed(PacketLength);

				TotalBytesRead -= PacketLength;
			}
		}
		else if (Socket->GetConnectionState() != ESocketConnectionState::SCS_Connected)
		{
			ConnectionLostDelegate.ExecuteIfBound();
			break;
		}

		// If the amount of time since the last packet was received from the server exceeds the timeout, assume that the communication flow is broken and attempt to reset it
		if ((FPlatformTime::Seconds() - LastTimeDataReceived) > DataReceivedTimeout)
		{
			// First, try a soft reset, which will re-initialize the command queue
			if (bSoftResetTriggered == false)
			{
				bSoftResetTriggered = true;
				SoftReset();
			}
			// If a soft reset does not work, assume that the connection to the socket is broken and trigger a hard reset to establish a new connection
			else if (bHardResetTriggered == false)
			{
				bHardResetTriggered = true;
				HardReset();
			}
		}
	}

	UE_LOGF(LogPrestonMDRMessageThread, Verbose, "Commands Sent   : %d", NumCommandsSent);
	UE_LOGF(LogPrestonMDRMessageThread, Verbose, "Replies Received: %d", NumRepliedReceived);

	bIsFinished = true;

	return 0;
}

void FPrestonMDRMessageThread::BuildStaticMessages()
{
	// Begin Status Message
	StatusRequestMessage.Reserve(FMDRProtocol::StatusMessageLength);

	StatusRequestMessage.Add(FMDRProtocol::STX);

	FAsciiByte StatusCommand = HexToAscii(static_cast<const uint8>(EMDRCommand::Status));
	StatusRequestMessage.Add(StatusCommand.ByteHigh);
	StatusRequestMessage.Add(StatusCommand.ByteLow);

	constexpr uint8 NumDataBytesInStatusRequest = 0;
	FAsciiByte NumStatusDataBytes = HexToAscii(NumDataBytesInStatusRequest);
	StatusRequestMessage.Add(NumStatusDataBytes.ByteHigh);
	StatusRequestMessage.Add(NumStatusDataBytes.ByteLow);

	FAsciiByte StatusChecksum = HexToAscii(ComputeChecksum(StatusRequestMessage.GetData(), StatusRequestMessage.Num()));
	StatusRequestMessage.Add(StatusChecksum.ByteHigh);
	StatusRequestMessage.Add(StatusChecksum.ByteLow);

	StatusRequestMessage.Add(FMDRProtocol::ETX);
	// End Status Message

	// Begin Data Message
	{
		// DataRequesetMessage may be written to from the GameThread when the data mode is changed
		FScopeLock Lock(&PrestonCriticalSection);

		DataRequestMessage.Reserve(FMDRProtocol::DataMessageLength);

		DataRequestMessage.Add(FMDRProtocol::STX);

		FAsciiByte DataCommand = HexToAscii(static_cast<const uint8>(EMDRCommand::Data));
		DataRequestMessage.Add(DataCommand.ByteHigh);
		DataRequestMessage.Add(DataCommand.ByteLow);

		constexpr uint8 NumDataBytesInDataRequest = 1;
		FAsciiByte NumDataBytes = HexToAscii(NumDataBytesInDataRequest);
		DataRequestMessage.Add(NumDataBytes.ByteHigh);
		DataRequestMessage.Add(NumDataBytes.ByteLow);

		// The data description and checksum bytes can be updated in place at a later time if the settings change
		// The default message uses the default settings, and thus is still a valid message to send to the MDR
		FAsciiByte DataDescription = HexToAscii(GetDataDescritionSettings());
		DataRequestMessage.Add(DataDescription.ByteHigh);
		DataRequestMessage.Add(DataDescription.ByteLow);

		FAsciiByte DataChecksum = HexToAscii(ComputeChecksum(DataRequestMessage.GetData(), DataRequestMessage.Num()));
		DataRequestMessage.Add(DataChecksum.ByteHigh);
		DataRequestMessage.Add(DataChecksum.ByteLow);

		DataRequestMessage.Add(FMDRProtocol::ETX);
	}
	// End Data Message

	// Begin Time Message
	TimeRequestMessage.Reserve(FMDRProtocol::TimeMessageLength);

	TimeRequestMessage.Add(FMDRProtocol::STX);

	FAsciiByte TimeCommand = HexToAscii(static_cast<const uint8>(EMDRCommand::Time));
	TimeRequestMessage.Add(TimeCommand.ByteHigh);
	TimeRequestMessage.Add(TimeCommand.ByteLow);

	constexpr uint8 NumDataBytesInTimeRequest = 1;
	FAsciiByte NumTimeDataBytes = HexToAscii(NumDataBytesInTimeRequest);
	TimeRequestMessage.Add(NumTimeDataBytes.ByteHigh);
	TimeRequestMessage.Add(NumTimeDataBytes.ByteLow);

	constexpr uint8 TimeDataDescription = 0x01; // See MDR Spec
	FAsciiByte TimeDescription = HexToAscii(TimeDataDescription);
	TimeRequestMessage.Add(TimeDescription.ByteHigh);
	TimeRequestMessage.Add(TimeDescription.ByteLow);

	FAsciiByte TimeChecksum = HexToAscii(ComputeChecksum(TimeRequestMessage.GetData(), TimeRequestMessage.Num()));
	TimeRequestMessage.Add(TimeChecksum.ByteHigh);
	TimeRequestMessage.Add(TimeChecksum.ByteLow);

	TimeRequestMessage.Add(FMDRProtocol::ETX);
	// End Time Message

	// Begin Timecode Message
	TimecodeRequestMessage.Reserve(FMDRProtocol::TimecodeMessageLength);

	TimecodeRequestMessage.Add(FMDRProtocol::STX);

	FAsciiByte TimecodeCommand = HexToAscii(static_cast<const uint8>(EMDRCommand::TimeCodeStatus));
	TimecodeRequestMessage.Add(TimecodeCommand.ByteHigh);
	TimecodeRequestMessage.Add(TimecodeCommand.ByteLow);

	constexpr uint8 NumDataBytesInTimecodeRequest = 0;
	FAsciiByte NumTimecodeDataBytes = HexToAscii(NumDataBytesInTimecodeRequest);
	TimecodeRequestMessage.Add(NumTimecodeDataBytes.ByteHigh);
	TimecodeRequestMessage.Add(NumTimecodeDataBytes.ByteLow);

	FAsciiByte TimecodeChecksum = HexToAscii(ComputeChecksum(TimecodeRequestMessage.GetData(), TimecodeRequestMessage.Num()));
	TimecodeRequestMessage.Add(TimecodeChecksum.ByteHigh);
	TimecodeRequestMessage.Add(TimecodeChecksum.ByteLow);

	TimecodeRequestMessage.Add(FMDRProtocol::ETX);
	// End Timecode Message
}

void FPrestonMDRMessageThread::UpdateDataDescriptionMessage_AnyThread()
{
	FScopeLock Lock(&PrestonCriticalSection);

	if (DataRequestMessage.Num() == FMDRProtocol::DataMessageLength)
	{
		constexpr uint32 DataDescriptionByte = FMDRProtocol::FirstPayloadByte - 1;
		FAsciiByte DataDescription = HexToAscii(GetDataDescritionSettings());
		DataRequestMessage[DataDescriptionByte + 0] = DataDescription.ByteHigh;
		DataRequestMessage[DataDescriptionByte + 1] = DataDescription.ByteLow;

		FAsciiByte DataChecksum = HexToAscii(ComputeChecksum(DataRequestMessage.GetData(), DataRequestMessage.Num() - FMDRProtocol::FooterSize));
		DataRequestMessage[DataRequestMessage.Num() + FMDRProtocol::ChecksumByte + 0] = DataChecksum.ByteHigh;
		DataRequestMessage[DataRequestMessage.Num() + FMDRProtocol::ChecksumByte + 1] = DataChecksum.ByteLow;
	}
}

void FPrestonMDRMessageThread::InitializeCommandQueue()
{
	CommandQueue.Empty();

	// Enqueue starting commands
	CommandQueue.Enqueue(EMDRCommand::Status);
	CommandQueue.Enqueue(EMDRCommand::TimeCodeStatus);

	// The main communication flow with the MDR involves:
	//   1) Sending a request for updated FIZ data
	//   2) Waiting for the reply from the MDR with FIZ
	//   3) Sending a request for an updated timecode
	//   4) Waiting for the reply from the MDR with timecode
	//   5) Repeat steps 1-4

	// These starting commands set up the initial flow. Every time a reply from the MDR is received in response to one of these requests, a new request is enqueued for the next frame
	CommandQueue.Enqueue(EMDRCommand::Data);
	CommandQueue.Enqueue(EMDRCommand::Time);

	bReadyToSendCommandToServer = true;
}

void FPrestonMDRMessageThread::SendNextCommandToServer()
{
	EMDRCommand NextCommand = EMDRCommand::Null;

	if (CommandQueue.Dequeue(NextCommand) == false)
	{
		UE_LOGF(LogPrestonMDRMessageThread, VeryVerbose, "Could not send any new requests to the MDR because the command queue was empty.");
		return;
	}

	if (NextCommand == EMDRCommand::Data)
	{
		// DataRequesetMessage may be written to from the GameThread when the data mode is changed
		FScopeLock Lock(&PrestonCriticalSection);
		SendMessageToServer(DataRequestMessage.GetData(), DataRequestMessage.Num());
	}
	else if (NextCommand == EMDRCommand::Status)
	{
		SendMessageToServer(StatusRequestMessage.GetData(), StatusRequestMessage.Num());
	}
	else if (NextCommand == EMDRCommand::Time)
	{
		SendMessageToServer(TimeRequestMessage.GetData(), TimeRequestMessage.Num());
	}
	else if (NextCommand == EMDRCommand::TimeCodeStatus)
	{
		SendMessageToServer(TimecodeRequestMessage.GetData(), TimecodeRequestMessage.Num());
	}
}

void FPrestonMDRMessageThread::SendMessageToServer(const uint8* const InMessageToSend, const int32 InNumBytesToSend)
{
	int32 NumBytesSent = 0;
	Socket->Send(InMessageToSend, InNumBytesToSend, NumBytesSent);

	// If the full message was not sent, make a second attempt to send the rest of the message
	if (NumBytesSent != InNumBytesToSend)
	{
		int32 NumBytesSentOnSecondAttempt = 0;
		Socket->Send(InMessageToSend + NumBytesSent, InNumBytesToSend - NumBytesSent, NumBytesSentOnSecondAttempt);
		NumBytesSent += NumBytesSentOnSecondAttempt;

		// If the full message was still not sent, something unexpected went wrong, and a reset is required
		if (NumBytesSent != InNumBytesToSend)
		{
			HardReset();
		}
	}
}

void FPrestonMDRMessageThread::ParseMessageFromServer(const uint8* const InPacketData, const uint32 InNumPacketBytes)
{
	ensureMsgf((InPacketData[FMDRProtocol::ACKByte] == FMDRProtocol::ACK),
		TEXT("Error Parsing Packet from MDR: The first byte was expected to be ACK, but was %#x instead."), InPacketData[FMDRProtocol::ACKByte]);

	ensureMsgf((InPacketData[FMDRProtocol::STXByte] == FMDRProtocol::STX),
		TEXT("Error Parsing Packet from MDR: The second byte was expected to be STX, but was %#x instead."), InPacketData[FMDRProtocol::STXByte]);

	EMDRCommand Command = EMDRCommand(AsciiToHex(&InPacketData[FMDRProtocol::CommandByte]));
	uint8 NumPayloadBytes = AsciiToHex(&InPacketData[FMDRProtocol::PayloadLengthByte]);

	switch (Command)
	{
	case EMDRCommand::Status:
	{
		ensureMsgf((NumPayloadBytes == FMDRProtocol::NumPayloadBytesInStatusMessage),
			TEXT("Error Parsing Status Message from MDR: Incorrect number of data bytes"));

		ParseStatusMessage(InPacketData);

		// The MDR's status message indicates which motors are connected. 
		// The data description message is therefore updated so that FIZ data is not requested for any motors that are not connected.
		UpdateDataDescriptionMessage_AnyThread();

		// A change in motor status should trigger a change in what FIZ data is supported by the Source
		StatusChangedDelegate.ExecuteIfBound(MDR3Status);
		break;
	}
	case EMDRCommand::Data:
	{
		ParseDataMessage(InPacketData, NumPayloadBytes);

		// Indicate that a new frame of FIZ data has been received
		bIsFrameDataReady = true;

		// Enqueue another FIZ Data command for the next frame
		CommandQueue.Enqueue(Command);
		break;
	}
	case EMDRCommand::Time:
	{
		ensureMsgf((NumPayloadBytes == FMDRProtocol::NumPayloadBytesInTimeMessage),
			TEXT("Error Parsing Time Message from MDR: Incorrect number of data bytes"));

		ParseTimeMessage(InPacketData);

		// Indicate that a new timecode has been received
		bIsFrameTimeReady = true;

		// Enqueue another Time command for the next timecode
		CommandQueue.Enqueue(Command);
		break;
	}
	case EMDRCommand::TimeCodeStatus:
	{
		ensureMsgf((NumPayloadBytes == FMDRProtocol::NumPayloadBytesInTimecodeMessage),
			TEXT("Error Parsing Timecode Status Message from MDR: Incorrect number of data bytes"));

		ParseTimecodeStatusMessage(InPacketData);
		break;
	}
	}

	uint8 Checksum = AsciiToHex(&InPacketData[InNumPacketBytes + FMDRProtocol::ChecksumByte]);
	ensureMsgf((Checksum == ComputeChecksum(InPacketData + 1, InNumPacketBytes - FMDRProtocol::FooterSize - 1)),
		TEXT("Error Parsing Packet from MDR: Checksum failed"));

	ensureMsgf((InPacketData[InNumPacketBytes + FMDRProtocol::ETXByte] == FMDRProtocol::ETX),
		TEXT("Error Parsing Packet from MDR: The last byte was expected to be ETX, but instead was %x"), InPacketData[InNumPacketBytes + FMDRProtocol::ETXByte]);
}

void FPrestonMDRMessageThread::ParseStatusMessage(const uint8* const InData)
{
	uint8 StatusByte0 = AsciiToHex(&InData[FMDRProtocol::FirstPayloadByte]);
	uint8 StatusByte1 = AsciiToHex(&InData[FMDRProtocol::FirstPayloadByte + 2]);

	MDR3Status.bIsAuxMotorSet = StatusByte0 & 0x80;
	MDR3Status.ZoomControl = ((StatusByte0 & 0x08) > 0) ? EZoomControl::Position : EZoomControl::Velocity;
	MDR3Status.AuxMotorAssignment = static_cast<EAuxMotorAssignment>(((StatusByte0 & 0x10) >> 1) + (StatusByte0 & 0x07));

	MDR3Status.bIsFocusCalibrated = StatusByte1 & 0x80;
	MDR3Status.bIsFocusAssistDetected = StatusByte1 & 0x20;
	MDR3Status.CameraRunStop = ((StatusByte1 & 0x10) > 0) ? ECameraRunStop::Run : ECameraRunStop::Stop;
	MDR3Status.bIsCameraSelected = StatusByte1 & 0x08;
	MDR3Status.bIsZoomMotorSet = StatusByte1 & 0x04;
	MDR3Status.bIsFocusMotorSet = StatusByte1 & 0x02;
	MDR3Status.bIsIrisMotorSet = StatusByte1 & 0x01;

	DataDescriptionSettings.bContainsZoom = MDR3Status.bIsZoomMotorSet;
	DataDescriptionSettings.bContainsFocus = MDR3Status.bIsFocusMotorSet;
	DataDescriptionSettings.bContainsIris = MDR3Status.bIsIrisMotorSet;
}

void FPrestonMDRMessageThread::ParseDataMessage(const uint8* const InData, const uint8 InNumDataBytes)
{
	uint8 DataDescription = AsciiToHex(&InData[FMDRProtocol::FirstPayloadByte]);

	// Each bit in the data description byte represents a data field that follows in the message
	const uint32 NumDataFields = FGenericPlatformMath::CountBits(DataDescription & 0x3F); // Ignore the MDRStatus
	constexpr uint32 NumBytesPerDataField = 2;

	ensureMsgf((NumDataFields * NumBytesPerDataField) == (InNumDataBytes - 1),
		TEXT("Error Parsing Data Message from MDR: Incorrect number of data bytes"));

	// Each data field in this message is encoded as 4 ASCII bytes, and decoded as a 16-bit integer
	uint32 DataPosition = FMDRProtocol::FirstPayloadByte + 2;
	for (uint32 Index = 0; Index < NumDataFields; ++Index)
	{
		const uint8 DataByte0 = AsciiToHex(&InData[DataPosition + 0]);
		const uint8 DataByte1 = AsciiToHex(&InData[DataPosition + 2]);
		ParsedFIZData[Index] = (DataByte0 << 8) + DataByte1;
		DataPosition += 4;
	}

	uint32 ParsedDataIndex = 0;

	// Data received is encoder positions
	if (DataDescription & static_cast<uint8>(EDataMessageBitmask::Mode))
	{
		if (DataDescription & static_cast<uint8>(EDataMessageBitmask::Iris))
		{
			LensData.Iris = ParsedFIZData[ParsedDataIndex];
			ParsedDataIndex++;
		}
		if (DataDescription & static_cast<uint8>(EDataMessageBitmask::Focus))
		{
			LensData.Focus = ParsedFIZData[ParsedDataIndex];
			ParsedDataIndex++;
		}
		if (DataDescription & static_cast<uint8>(EDataMessageBitmask::Zoom))
		{
			LensData.Zoom = ParsedFIZData[ParsedDataIndex];
			ParsedDataIndex++;
		}
		if (DataDescription & static_cast<uint8>(EDataMessageBitmask::AUX))
		{
			LensData.Aux = ParsedFIZData[ParsedDataIndex];
			ParsedDataIndex++;
		}
	}
	// Data received is pre-calibrated FIZ
	else
	{
		if (DataDescription & static_cast<uint8>(EDataMessageBitmask::Iris))
		{
			LensData.Iris = (ParsedFIZData[ParsedDataIndex] / 100.0f);
			ParsedDataIndex++;
		}
		if (DataDescription & static_cast<uint8>(EDataMessageBitmask::Focus))
		{
			float FocusFeet = ParsedFIZData[ParsedDataIndex] / 100.0f; // In feet
			LensData.Focus = FocusFeet * 30.48; // feet -> cm conversion
			ParsedDataIndex++;
		}
		if (DataDescription & static_cast<uint8>(EDataMessageBitmask::Zoom))
		{
			LensData.Zoom = ParsedFIZData[ParsedDataIndex];
			ParsedDataIndex++;
		}
		if (DataDescription & static_cast<uint8>(EDataMessageBitmask::AUX))
		{
			LensData.Aux = ParsedFIZData[ParsedDataIndex];
			ParsedDataIndex++;
		}
	}
}

void FPrestonMDRMessageThread::ParseTimeMessage(const uint8* const InData)
{
	constexpr uint32 FrameNumberBytePosition = FMDRProtocol::FirstPayloadByte + 2;
	constexpr uint32 SecondsBytePosition = FMDRProtocol::FirstPayloadByte + 4;
	constexpr uint32 MinutesBytePosition = FMDRProtocol::FirstPayloadByte + 6;
	constexpr uint32 HoursBytePosition = FMDRProtocol::FirstPayloadByte + 8;

	uint8 Frames = AsciiToDec(&InData[FrameNumberBytePosition]);
	uint8 Seconds = AsciiToDec(&InData[SecondsBytePosition]);
	uint8 Minutes = AsciiToDec(&InData[MinutesBytePosition]);
	uint8 Hours = AsciiToDec(&InData[HoursBytePosition]);

	FTimecode Timecode = FTimecode(Hours, Minutes, Seconds, Frames, bIsDropFrameRate);
	LensData.FrameTime = FQualifiedFrameTime(Timecode, MDRFrameRate);
}

void FPrestonMDRMessageThread::ParseTimecodeStatusMessage(const uint8* const InData)
{
	uint8 TimecodeStatus = AsciiToHex(&InData[FMDRProtocol::FirstPayloadByte]);

	uint8 FrameRate = (TimecodeStatus & 0x07);
	if (FrameRate == 0) { MDRFrameRate = FFrameRate(24, 1); }
	else if (FrameRate == 1) { MDRFrameRate = FFrameRate(24000, 1001); }
	else if (FrameRate == 2) { MDRFrameRate = FFrameRate(24, 1); }
	else if (FrameRate == 3) { MDRFrameRate = FFrameRate(25, 1); }
	else if (FrameRate == 4) { MDRFrameRate = FFrameRate(30000, 1001); }
	else if (FrameRate == 5) { MDRFrameRate = FFrameRate(30, 1); }

	bIsDropFrameRate = (TimecodeStatus & 0x80) > 0;
}

void FPrestonMDRMessageThread::SoftReset()
{
	InitializeCommandQueue();

	// Reset the timer so that another reset does not immediately trigger
	LastTimeDataReceived = FPlatformTime::Seconds();
}

void FPrestonMDRMessageThread::HardReset()
{
	// Update this flag so that the main message loop stops executing and the thread can be marked as finished
	bForceKillThread = true;

	// Notify the Source that the connection has been lost so that it can attempt to re-establish it.
	ConnectionLostDelegate.ExecuteIfBound();
}

void FPrestonMDRMessageThread::ForceKill()
{
	// Update this flag so that the main message loop stops executing and the thread can be marked as finished
	bForceKillThread = true;
}

// Utility Functions
FAsciiByte FPrestonMDRMessageThread::HexToAscii(const uint8 InHexByte)
{
	FAsciiByte AsciiResult;

	uint8 HexByteHigh = (InHexByte >> 4);
	uint8 HexByteLow = (InHexByte & 0x0F);

	if ((HexByteHigh >= 0) && (HexByteHigh <= 9))
	{
		AsciiResult.ByteHigh = HexByteHigh + 0x30;
	}
	else if ((HexByteHigh >= 0x0A) && (HexByteHigh <= 0x0F))
	{
		AsciiResult.ByteHigh = HexByteHigh + 0x37;
	}

	if ((HexByteLow >= 0) && (HexByteLow <= 9))
	{
		AsciiResult.ByteLow = HexByteLow + 0x30;
	}
	else if ((HexByteLow >= 0x0A) && (HexByteLow <= 0x0F))
	{
		AsciiResult.ByteLow = HexByteLow + 0x37;
	}

	return AsciiResult;
}

uint8 FPrestonMDRMessageThread::AsciiToHex(const FAsciiByte InAsciiByte)
{
	uint8 HexByteHigh = 0x00;
	uint8 HexByteLow = 0x00;

	if ((InAsciiByte.ByteHigh >= '0') && (InAsciiByte.ByteHigh <= '9'))
	{
		HexByteHigh = InAsciiByte.ByteHigh - 0x30;
	}
	else if ((InAsciiByte.ByteHigh >= 'A') && (InAsciiByte.ByteHigh <= 'F'))
	{
		HexByteHigh = InAsciiByte.ByteHigh - 0x37;
	}

	if ((InAsciiByte.ByteLow >= '0') && (InAsciiByte.ByteLow <= '9'))
	{
		HexByteLow = InAsciiByte.ByteLow - 0x30;
	}
	else if ((InAsciiByte.ByteLow >= 'A') && (InAsciiByte.ByteLow <= 'F'))
	{
		HexByteLow = InAsciiByte.ByteLow - 0x37;
	}

	return (HexByteHigh << 4) + HexByteLow;
}

uint8 FPrestonMDRMessageThread::AsciiToDec(const FAsciiByte InAsciiByte)
{
	uint8 DecByteHigh = 0x00;
	uint8 DecByteLow = 0x00;

	if ((InAsciiByte.ByteHigh >= '0') && (InAsciiByte.ByteHigh <= '9'))
	{
		DecByteHigh = InAsciiByte.ByteHigh - 0x30;
	}
	else if ((InAsciiByte.ByteHigh >= 'A') && (InAsciiByte.ByteHigh <= 'F'))
	{
		DecByteHigh = InAsciiByte.ByteHigh - 0x37;
	}

	if ((InAsciiByte.ByteLow >= '0') && (InAsciiByte.ByteLow <= '9'))
	{
		DecByteLow = InAsciiByte.ByteLow - 0x30;
	}
	else if ((InAsciiByte.ByteLow >= 'A') && (InAsciiByte.ByteLow <= 'F'))
	{
		DecByteLow = InAsciiByte.ByteLow - 0x37;
	}

	return (DecByteHigh * 10) + DecByteLow;
}

uint8 FPrestonMDRMessageThread::ComputeChecksum(const uint8* const InData, const uint8 InDataLength)
{
	uint8 Checksum = 0;
	for (uint32 Index = 0; Index < InDataLength; ++Index)
	{
		Checksum += InData[Index];
	}
	return Checksum;
}

uint8 FPrestonMDRMessageThread::GetDataDescritionSettings()
{
	uint8 Result = 0;
	Result = DataDescriptionSettings.bContainsMDRStatus ? Result | static_cast<uint8>(EDataMessageBitmask::Status) : Result;
	Result = (DataDescriptionSettings.LensDataMode == EFIZDataMode::EncoderData) ? Result | static_cast<uint8>(EDataMessageBitmask::Mode) : Result;
	Result = DataDescriptionSettings.bContainsDistance ? Result | static_cast<uint8>(EDataMessageBitmask::Distance) : Result;
	Result = DataDescriptionSettings.bContainsSpeed ? Result | static_cast<uint8>(EDataMessageBitmask::Speed) : Result;
	Result = DataDescriptionSettings.bContainsAux ? Result | static_cast<uint8>(EDataMessageBitmask::AUX) : Result;
	Result = DataDescriptionSettings.bContainsZoom ? Result | static_cast<uint8>(EDataMessageBitmask::Zoom) : Result;
	Result = DataDescriptionSettings.bContainsFocus ? Result | static_cast<uint8>(EDataMessageBitmask::Focus) : Result;
	Result = DataDescriptionSettings.bContainsIris ? Result | static_cast<uint8>(EDataMessageBitmask::Iris) : Result;
	return Result;
}

