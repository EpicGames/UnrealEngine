// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "HAL/Runnable.h"
#include "LiveLinkPrestonMDRSourceSettings.h"

class FRunnable;
class FSocket;

struct FLensDataPacket;
struct FMDR3Status;

DECLARE_DELEGATE_OneParam(FOnFrameDataReady, FLensDataPacket InData);
DECLARE_DELEGATE_OneParam(FOnStatusChanged, FMDR3Status InStatus);
DECLARE_DELEGATE(FOnConnectionLost)
DECLARE_DELEGATE(FOnConnectionFailed)

enum class EDataMessageBitmask : uint8
{
	Iris = 0x01,
	Focus = 0x02,
	Zoom = 0x04,
	AUX = 0x08,
	Speed = 0x10,
	Distance = 0x20,
	Mode = 0x40,
	Status = 0x80
};

enum EAuxMotorAssignment : uint8
{
	None = 0,
	HandUnit3Focus = 1,
	HandUnit3Iris = 2,
	HandUnit3Zoom = 3,
	WirelessFocus = 4,
	WirelessIris = 5,
	RadioMF = 6,
	AnalogFocus = 7,
	AnalogIris = 8,
	AnalogZoom = 9
};

enum class EZoomControl : uint8
{
	Velocity = 0,
	Position = 1
};

enum class ECameraRunStop : uint8
{
	Stop = 0,
	Run = 1
};

enum class EMDRCommand : uint8
{
	Null = 0x00,
	Status = 0x02,
	Data = 0x04,
	Time = 0x05,
	TimeCodeStatus = 0x0B,
	Info = 0x0E,
	Error = 0x11,
};

// Preston MDR Protocol Constants
struct FMDRProtocol
{
	// ASCII Constants
	static const uint8 STX;
	static const uint8 ETX;
	static const uint8 ACK;

	// Begin Header Bytes
	static constexpr uint32 ACKByte = 0;
	static constexpr uint32 STXByte = ACKByte + 1;
	static constexpr uint32 CommandByte = STXByte + 1;
	static constexpr uint32 PayloadLengthByte = CommandByte + 2;
	//~ End Header Bytes

	static constexpr uint32 FirstPayloadByte = PayloadLengthByte + 2;

	// Begin Footer Bytes
	static constexpr uint32 ETXByte = -1;
	static constexpr uint32 ChecksumByte = ETXByte - 2;
	//~ End Footer Bytes

	static constexpr uint32 HeaderSize = 6;
	static constexpr uint32 FooterSize = 3;

	// These values represent the static length of each request message type
	static const uint8 StatusMessageLength;
	static const uint8 DataMessageLength;
	static const uint8 TimeMessageLength;
	static const uint8 TimecodeMessageLength;

	// These values represent the static length of the payload in each message type
	static constexpr uint32 NumPayloadBytesInStatusMessage = 2;
	static constexpr uint32 NumPayloadBytesInTimeMessage = 9;
	static constexpr uint32 NumPayloadBytesInTimecodeMessage = 1;
};

struct FLensDataPacket
{
	FQualifiedFrameTime FrameTime;

	float Focus = 0.0f;
	float Iris = 0.0f;
	float Zoom = 0.0f;
	float Aux = 0.0f;
};

struct FMDR3Status
{
	bool bIsAuxMotorSet = false;
	bool bIsZoomMotorSet = false;
	bool bIsFocusMotorSet = false;
	bool bIsIrisMotorSet = false;
	bool bIsCameraSelected = false;
	ECameraRunStop CameraRunStop = ECameraRunStop::Stop;
	bool bIsFocusAssistDetected = false;
	bool bIsFocusCalibrated = false;
	EAuxMotorAssignment AuxMotorAssignment = EAuxMotorAssignment::None;
	EZoomControl ZoomControl = EZoomControl::Position;
};

// Settings used to encode which pieces of data the MDR should send in a "DATA" message
struct FDataMessageDescription
{
	bool bContainsMDRStatus = false;
	EFIZDataMode LensDataMode = EFIZDataMode::EncoderData;
	bool bContainsDistance = false;
	bool bContainsSpeed = false;
	bool bContainsAux = false;
	bool bContainsZoom = false;
	bool bContainsFocus = false;
	bool bContainsIris = false;
};

struct FAsciiByte
{
	uint8 ByteHigh = 0;
	uint8 ByteLow = 0;

	FAsciiByte() {}

	FAsciiByte(const uint8* const InData)
	{
		ByteHigh = InData[0];
		ByteLow = InData[1];
	}
};

class FPrestonMDRMessageThread : public FRunnable
{
public:

	FPrestonMDRMessageThread(FSocket* InSocket);
	~FPrestonMDRMessageThread();

	void Start();

	bool IsThreadRunning() const { return bIsThreadRunning; }
	bool IsFinished() const { return bIsFinished; }

	/**
	 * Returns a delegate that is executed when frame data is ready.
	 *
	 * @return The delegate.
	 */
	FOnFrameDataReady& OnFrameDataReady()
	{
		return FrameDataReadyDelegate;
	}

	FOnStatusChanged& OnStatusChanged()
	{
		return StatusChangedDelegate;
	}

	FOnConnectionLost& OnConnectionLost()
	{
		return ConnectionLostDelegate;
	}

	FOnConnectionFailed& OnConnectionFailed()
	{
		return ConnectionFailedDelegate;
	}

public:

	//~ FRunnable Interface
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	// End FRunnable Interface

	void SetSocket(FSocket* InSocket);

	void SoftReset();

	void ForceKill();

	void SetIncomingDataMode_GameThread(EFIZDataMode InDataMode);

private:

	void SendNextCommandToServer();

	void ParseMessageFromServer(const uint8* const InPacketData, const uint32 InNumPacketBytes);
	void SendMessageToServer(const uint8* const InMessageToSend, const int32 InNumBytesToSend);

	void ParseStatusMessage(const uint8* const InData);
	void ParseDataMessage(const uint8* const InData, const uint8 InNumDataBytes);
	void ParseTimeMessage(const uint8* const InData);
	void ParseTimecodeStatusMessage(const uint8* const InData);

	void BuildStaticMessages();
	void UpdateDataDescriptionMessage_AnyThread();
	void InitializeCommandQueue();
	void HardReset();

	// Conversion functions to translate between data bytes and ASCII-encoded bytes
	FAsciiByte HexToAscii(const uint8 InHexByte);
	uint8 AsciiToHex(const FAsciiByte InAsciiByte);
	uint8 AsciiToDec(const FAsciiByte InAsciiByte);

	uint8 ComputeChecksum(const uint8* const InData, const uint8 InDataLength);
	uint8 GetDataDescritionSettings();

private:
	FSocket* Socket;

	TUniquePtr<FRunnableThread> Thread;
	std::atomic<bool> bIsThreadRunning = false;

	FOnFrameDataReady FrameDataReadyDelegate;
	FOnStatusChanged StatusChangedDelegate;
	FOnConnectionLost ConnectionLostDelegate;
	FOnConnectionFailed ConnectionFailedDelegate;

	FCriticalSection PrestonCriticalSection;

	TQueue<EMDRCommand> CommandQueue;
	TArray<uint8> StatusRequestMessage;
	TArray<uint8> DataRequestMessage;
	TArray<uint8> TimeRequestMessage;
	TArray<uint8> TimecodeRequestMessage;

	FMDR3Status MDR3Status;
	FLensDataPacket LensData;

	FFrameRate MDRFrameRate = { 24, 1 };
	bool bIsDropFrameRate = false;

	FDataMessageDescription DataDescriptionSettings;
	bool bIsFrameDataReady = false;
	bool bIsFrameTimeReady = false;
	bool bReadyToSendCommandToServer = false;

	TArray<uint16> ParsedFIZData;

	double LastTimeDataReceived = 0.0;

	bool bSoftResetTriggered = false;
	bool bHardResetTriggered = false;

	std::atomic<bool> bIsFinished = true;
	std::atomic<bool> bForceKillThread = false;

private:
	static constexpr uint32 MaximumMDRMessageLength = 512;

	static constexpr uint32 ThreadStackSize = 1024 * 128;

	static constexpr float ConnectionWaitInterval = 0.02f; // 20ms
	static constexpr double ConnectionTimeout = 5.0; // 3sec
	static constexpr double DataReceivedTimeout = 2.0; // 2sec
};
