// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkPrestonMDRConnectionSettings.generated.h"

USTRUCT()
struct FLiveLinkPrestonMDRConnectionSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings")
	FString IPAddress = TEXT("0.0.0.0");

	UPROPERTY(EditAnywhere, Category = "Settings")
	uint16 PortNumber = 0;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName SubjectName = TEXT("Preston MDR");
};
