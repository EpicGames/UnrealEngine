// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkSourceFactory.h"

#include "LiveLinkPrestonMDRFactory.generated.h"

class SLiveLinkPrestonMDRSourcePanel;
struct FLiveLinkPrestonMDRConnectionSettings;

UCLASS()
class LIVELINKPRESTONMDR_API ULiveLinkPrestonMDRSourceFactory : public ULiveLinkSourceFactory
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedPtr<SWidget>, FBuildCreationPanelDelegate, const ULiveLinkPrestonMDRSourceFactory*, FOnLiveLinkSourceCreated);
	static FBuildCreationPanelDelegate OnBuildCreationPanel;

public:
	GENERATED_BODY()

	//~ Begin ULiveLinkSourceFactory interface
	virtual FText GetSourceDisplayName() const;
	virtual FText GetSourceTooltip() const;

	virtual EMenuType GetMenuType() const override { return EMenuType::SubPanel; }
	virtual TSharedPtr<SWidget> BuildCreationPanel(FOnLiveLinkSourceCreated OnLiveLinkSourceCreated) const override;
	virtual TSharedPtr<ILiveLinkSource> CreateSource(const FString& ConnectionString) const override;
	//~ End ULiveLinkSourceFactory interface

	static FString CreateConnectionString(const FLiveLinkPrestonMDRConnectionSettings& Settings);
};
