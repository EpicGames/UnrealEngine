// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkPrestonMDREditorModule.h"

#include "LiveLinkPrestonMDRFactory.h"
#include "LiveLinkPrestonMDRSourcePanel.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "LiveLinkPrestonMDREditorModule"

void FLiveLinkPrestonMDREditorModule::StartupModule()
{
	// register the LiveLinkFactory panel
	auto BuildCreationPanel = [](const ULiveLinkPrestonMDRSourceFactory* Factory, ULiveLinkSourceFactory::FOnLiveLinkSourceCreated OnSourceCreated)
		-> TSharedPtr<SWidget>
	{
		return SNew(SLiveLinkPrestonMDRSourcePanel)
			.Factory(Factory)
			.OnSourceCreated(OnSourceCreated);
	};
	ULiveLinkPrestonMDRSourceFactory::OnBuildCreationPanel.BindLambda(BuildCreationPanel);
}

void FLiveLinkPrestonMDREditorModule::ShutdownModule()
{
	ULiveLinkPrestonMDRSourceFactory::OnBuildCreationPanel.Unbind();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FLiveLinkPrestonMDREditorModule, LiveLinkPrestonMDREditor)
