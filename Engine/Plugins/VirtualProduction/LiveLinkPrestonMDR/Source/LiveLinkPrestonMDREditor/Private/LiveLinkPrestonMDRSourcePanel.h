// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkPrestonMDRConnectionSettings.h"

#include "LiveLinkSourceFactory.h"
#include "Widgets/SCompoundWidget.h"

class ULiveLinkPrestonMDRSourceFactory;

class SLiveLinkPrestonMDRSourcePanel : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLiveLinkPrestonMDRSourcePanel) {}
		SLATE_ARGUMENT(const ULiveLinkPrestonMDRSourceFactory*, Factory)
		SLATE_EVENT(ULiveLinkSourceFactory::FOnLiveLinkSourceCreated, OnSourceCreated)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);

public:
	FReply CreateNewSource(bool bShouldCreateSource);

private:
	FLiveLinkPrestonMDRConnectionSettings ConnectionSettings;
	TWeakObjectPtr<const ULiveLinkPrestonMDRSourceFactory> SourceFactory;
	ULiveLinkSourceFactory::FOnLiveLinkSourceCreated OnSourceCreated;
};
